{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Facial Animation Bulk Importer",
	"Description" : "Bulk importer for facial animation curves and audio. Imports facial animation curve tables (from FBX) into sound waves.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "FacialAnimation",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "FacialAnimationEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PreDefault"
		}	
	]
}