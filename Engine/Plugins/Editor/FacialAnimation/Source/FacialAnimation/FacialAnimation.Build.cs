// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FacialAnimation : ModuleRules
	{
		public FacialAnimation(ReadOnlyTargetRules Target) : base(Target)
		{			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"InputCore",
					"Engine",
					"AudioExtensions",
					"AudioMixer"
				}
			);
		}
	}
}
