// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCurveSourceComponent.h"
#include "Sound/SoundWave.h"
#include "Engine/CurveTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioCurveSourceComponent)

UAudioCurveSourceComponent::UAudioCurveSourceComponent()
{
	OnAudioPlaybackPercentNative.AddUObject(this, &UAudioCurveSourceComponent::HandlePlaybackPercent);

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;

	static FName DefaultSourceBindingName(TEXT("Default"));
	CurveSourceBindingName = DefaultSourceBindingName;

	CachedCurveEvalTime = 0.0f;
	CachedSyncPreRoll = 0.0f;
	CachedStartTime = 0.0f;
	CachedFadeInDuration = 0.0f;
	CachedFadeVolumeLevel = 1.0f;
	CachedDuration = 0.0f;
	bCachedLooping = false;
	CachedFadeType = EAudioFaderCurve::Linear;

#if WITH_EDITORONLY_DATA
	bVisualizeComponent = false;
#endif
}

void UAudioCurveSourceComponent::CacheCurveData()
{
	CachedCurveTable = nullptr;
	CachedCurveEvalTime = 0.0f;
	CachedSyncPreRoll = 0.0f;
	CachedDuration = 0.0f;
	bCachedLooping = false;

	// we only support preroll with sound waves (and derived classes) as these are the
	// only types where we can properly determine correct sound waves up-front (sound 
	// cues etc. can be randomized etc.)
	if (UCurveTable* CurveData = Sound ? Sound->GetCurveData() : nullptr)
	{
		CachedCurveTable = CurveData;

		// cache audio sync curve
		static const FName AudioSyncCurve(TEXT("Audio"));
		if (FRealCurve* Curve = CurveData->FindCurve(AudioSyncCurve, FString(), false))
		{
			CachedSyncPreRoll = Curve->GetKeyTime(Curve->GetFirstKeyHandle());
		}

		CachedDuration = Sound->Duration;
		bCachedLooping = Sound->IsLooping();
	}
}

void UAudioCurveSourceComponent::FadeIn(float FadeInDuration, float FadeVolumeLevel, float StartTime, const EAudioFaderCurve FadeType)
{
	CacheCurveData();

	if (CachedSyncPreRoll <= 0.0f)
	{
		PlayInternalRequestData PlayData;
		PlayData.StartTime = StartTime;
		PlayData.FadeInDuration = FadeInDuration;
		PlayData.FadeVolumeLevel = FadeVolumeLevel;
		PlayData.FadeCurve = FadeType;

		PlayInternal(PlayData);
	}
	else
	{
		CachedFadeInDuration = FadeInDuration;
		CachedFadeVolumeLevel = FadeVolumeLevel;
		CachedStartTime = StartTime;
		CachedFadeType = FadeType;
		Delay = 0.0f;
	}
}

void UAudioCurveSourceComponent::FadeOut(float FadeOutDuration, float FadeVolumeLevel, const EAudioFaderCurve FadeType)
{
	if (Delay < CachedSyncPreRoll)
	{
		CachedCurveTable = nullptr;
		CachedCurveEvalTime = 0.0f;
		CachedSyncPreRoll = 0.0f;
		Delay = 0.0f;
	}
	else
	{
		Super::FadeOut(FadeOutDuration, FadeVolumeLevel, FadeType);
	}
}

void UAudioCurveSourceComponent::Play(float StartTime)
{
	CacheCurveData();

	if (CachedSyncPreRoll <= 0.0f)
	{
		PlayInternalRequestData PlayData;
		PlayData.StartTime = StartTime;

		PlayInternal(PlayData);
	}
	else
	{
		CachedFadeInDuration = 0.0f;
		CachedFadeVolumeLevel = 1.0f;
		CachedStartTime = 0.0f;
		Delay = 0.0f;
	}
}

void UAudioCurveSourceComponent::Stop()
{
	if (Delay < CachedSyncPreRoll)
	{
		CachedCurveTable = nullptr;
		CachedCurveEvalTime = 0.0f;
		CachedSyncPreRoll = 0.0f;
		Delay = 0.0f;
	}
	else
	{
		Super::Stop();
	}
}

bool UAudioCurveSourceComponent::IsPlaying() const
{
	return Delay < CachedSyncPreRoll || Super::IsPlaying();
}

void UAudioCurveSourceComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	const float OldDelay = Delay;
	Delay = FMath::Min(Delay + DeltaTime, CachedSyncPreRoll);
	if (OldDelay < CachedSyncPreRoll && Delay >= CachedSyncPreRoll)
	{
		PlayInternalRequestData PlayData;
		PlayData.StartTime = CachedStartTime;
		PlayData.FadeInDuration = CachedFadeInDuration;
		PlayData.FadeVolumeLevel = CachedFadeVolumeLevel;
		PlayData.FadeCurve = CachedFadeType;

		PlayInternal(PlayData);
	}
	else if(Delay < CachedSyncPreRoll)
	{
		CachedCurveEvalTime = Delay;
	}
}

void UAudioCurveSourceComponent::HandlePlaybackPercent(const UAudioComponent* InComponent, const USoundWave* InSoundWave, const float InPlaybackPercentage)
{
	CachedCurveTable = InSoundWave->GetCurveData();
	CachedDuration = InSoundWave->Duration;
	CachedCurveEvalTime = CurveSyncOffset + Delay + (InPlaybackPercentage * InSoundWave->Duration);
	bCachedLooping = const_cast<USoundWave*>(InSoundWave)->IsLooping();
}

FName UAudioCurveSourceComponent::GetBindingName_Implementation() const
{
	return CurveSourceBindingName;
}

float UAudioCurveSourceComponent::GetCurveValue_Implementation(FName CurveName) const
{
	if (IsPlaying())
	{
		UCurveTable* CurveTable = CachedCurveTable.Get();
		if (CurveTable)
		{
			if (FRealCurve* Curve = CurveTable->FindCurve(CurveName, FString(), false))
			{
				return Curve->Eval(CachedCurveEvalTime);
			}
		}
	}

	return 0.0f;
}

void UAudioCurveSourceComponent::GetCurves_Implementation(TArray<FNamedCurveValue>& OutCurve) const
{
	if (IsPlaying())
	{
		UCurveTable* CurveTable = CachedCurveTable.Get();
		if (CurveTable)
		{
			OutCurve.Reset(CurveTable->GetRowMap().Num());

			if (bCachedLooping && CachedSyncPreRoll > 0.0f && Delay >= CachedSyncPreRoll && CachedCurveEvalTime >= CachedDuration - CachedSyncPreRoll)
			{
				// if we are looping and we have some preroll delay, we need to evaluate the curve twice
				// as we need to incorporate the preroll in the loop
				for (auto Iter = CurveTable->GetRowMap().CreateConstIterator(); Iter; ++Iter)
				{
					FRealCurve* Curve = Iter.Value();

					float StandardValue = Curve->Eval(CachedCurveEvalTime);
					float LoopedValue = Curve->Eval(FMath::Fmod(CachedCurveEvalTime, CachedDuration));

					OutCurve.Add({ Iter.Key(), FMath::Max(StandardValue, LoopedValue) });
				}
			}
			else
			{
				for (auto Iter = CurveTable->GetRowMap().CreateConstIterator(); Iter; ++Iter)
				{
					OutCurve.Add({ Iter.Key(), Iter.Value()->Eval(CachedCurveEvalTime) });
				}
			}
		}
	}
}
