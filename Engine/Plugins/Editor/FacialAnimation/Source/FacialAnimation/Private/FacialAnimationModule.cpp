// Copyright Epic Games, Inc. All Rights Reserved.

#include "FacialAnimationModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FFacialAnimationModule, FacialAnimation)

void FFacialAnimationModule::StartupModule()
{
}

void FFacialAnimationModule::ShutdownModule()
{
}
