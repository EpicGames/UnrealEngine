// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/AudioComponent.h"
#include "Animation/CurveSourceInterface.h"

#include "AudioCurveSourceComponent.generated.h"

#define UE_API FACIALANIMATION_API

class UCurveTable;
class USoundWave;

/** An audio component that also provides curves to drive animation */
UCLASS(MinimalAPI, ClassGroup = Audio, Experimental, meta = (BlueprintSpawnableComponent))
class UAudioCurveSourceComponent : public UAudioComponent, public ICurveSourceInterface
{
	GENERATED_BODY()

public:
	UE_API UAudioCurveSourceComponent();

	/** 
	 * Get the name that this curve source can be bound to by.
	 * Clients of this curve source will use this name to identify this source.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Curves)
	FName CurveSourceBindingName;

	/** Offset in time applied to audio position when evaluating curves */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Curves)
	float CurveSyncOffset;

public:
	/** UActorComponent interface */
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	/** UAudioComponent interface */
	UE_API virtual void FadeIn(float FadeInDuration, float FadeVolumeLevel = 1.0f, float StartTime = 0.0f, const EAudioFaderCurve FadeType = EAudioFaderCurve::Linear) override;
	UE_API virtual	void FadeOut(float FadeOutDuration, float FadeVolumeLevel, const EAudioFaderCurve FadeType = EAudioFaderCurve::Linear) override;
	UE_API virtual void Play(float StartTime = 0.0f) override;
	UE_API virtual void Stop() override;
	UE_API virtual bool IsPlaying() const override;

	/** ICurveSourceInterface interface */
	UE_API virtual FName GetBindingName_Implementation() const override;
	UE_API virtual float GetCurveValue_Implementation(FName CurveName) const override;
	UE_API virtual void GetCurves_Implementation(TArray<FNamedCurveValue>& OutCurve) const override;

private:
	/** Cache the curve parameters when playing back */
	void CacheCurveData();

	/** Internal handling of playback percentage */
	void HandlePlaybackPercent(const UAudioComponent* InComponent, const USoundWave* InSoundWave, const float InPlaybackPercentage);

private:
	/** Cached evaluation time from the last callback of OnPlaybackPercent */
	float CachedCurveEvalTime;

	/** Cached curve table from the last callback of OnPlaybackPercent */
	TWeakObjectPtr<class UCurveTable> CachedCurveTable;

	/** Preroll time we use to sync to curves */
	float CachedSyncPreRoll;

	/** Cached param for PlayInternal */
	float CachedStartTime;

	/** Cached param for PlayInternal */
	float CachedFadeInDuration;

	/** Cached param for PlayInternal */
	float CachedFadeVolumeLevel;

	/** Delay timer */
	float Delay;

	/** Cached duration */
	float CachedDuration;

	/** Cache looping flag */
	bool bCachedLooping;

	/** Cached param for PlayInternal */
	EAudioFaderCurve CachedFadeType;
};

#undef UE_API
