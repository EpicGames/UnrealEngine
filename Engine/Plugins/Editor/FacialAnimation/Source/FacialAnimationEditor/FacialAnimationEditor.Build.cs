// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FacialAnimationEditor : ModuleRules
	{
		public FacialAnimationEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
                    "Core",
                    "CoreUObject",
                    "InputCore",
                    "Engine",
                    "FacialAnimation",
                    "SlateCore",
                    "Slate",
					"EditorFramework",
					"UnrealEd",
                    "AudioEditor",
                    "WorkspaceMenuStructure",
                    "DesktopPlatform",
                    "PropertyEditor",
                    "TargetPlatform",
                    "Persona",
                }
			);
		}
	}
}
