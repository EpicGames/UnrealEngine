// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FacialAnimationBulkImporterSettings.generated.h"

UCLASS(Config=EditorPerProjectUserSettings, Experimental, MinimalAPI)
class UFacialAnimationBulkImporterSettings : public UObject
{
public:
	GENERATED_BODY()

	/** UObject interface */
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	/** The path to import files from */
	UPROPERTY(Config, EditAnywhere, Category = "Directories")
	FDirectoryPath SourceImportPath;

	/** The path to import files to */
	UPROPERTY(Config, EditAnywhere, Category = "Directories", meta=(ContentDir))
	FDirectoryPath TargetImportPath;

	/** Node in the FBX scene that contains the curves we are interested in */
	UPROPERTY(Config, EditAnywhere, Category = "Settings")
	FString CurveNodeName;
};
