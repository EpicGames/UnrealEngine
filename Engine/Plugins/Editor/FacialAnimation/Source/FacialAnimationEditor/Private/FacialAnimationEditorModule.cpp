// Copyright Epic Games, Inc. All Rights Reserved.

#include "FacialAnimationEditorModule.h"
#include "SFacialAnimationBulkImporter.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructureModule.h"
#include "FacialAnimationBulkImporterSettings.h"
#include "IPersonaPreviewScene.h"
#include "PersonaModule.h"
#include "AudioCurveSourceComponent.h"
#include "WorkspaceMenuStructure.h"
#include "Settings/EditorExperimentalSettings.h"

#define LOCTEXT_NAMESPACE "FacialAnimationEditor"

IMPLEMENT_MODULE(FFacialAnimationEditorModule, FacialAnimationEditor)

const FName FacialAnimationBulkImporterTabName(TEXT("FacialAnimationBulkImporter"));

static void CreatePersonaPreviewAudioComponent(const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	if (AActor* Actor = InPreviewScene->GetActor())
	{
		// Create the preview audio component
		UAudioCurveSourceComponent* AudioCurveSourceComponent = NewObject<UAudioCurveSourceComponent>(Actor);
		AudioCurveSourceComponent->bPreviewComponent = true;
		AudioCurveSourceComponent->bAlwaysPlay = true;
		AudioCurveSourceComponent->bIsPreviewSound = true;
		AudioCurveSourceComponent->bIsUISound = true;
		AudioCurveSourceComponent->CurveSourceBindingName = ICurveSourceInterface::DefaultBinding;

		InPreviewScene->AddComponent(AudioCurveSourceComponent, FTransform::Identity);
	}
}

void FFacialAnimationEditorModule::HandleModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason)
{
	if (InModuleName == TEXT("Persona") && InModuleChangeReason == EModuleChangeReason::ModuleLoaded)
	{
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
		OnPreviewSceneCreatedDelegate = PersonaModule.OnPreviewSceneCreated().AddStatic(&CreatePersonaPreviewAudioComponent);
	}
}

void FFacialAnimationEditorModule::StartupModule()
{
	GetMutableDefault<UFacialAnimationBulkImporterSettings>()->LoadConfig();

	struct Local
	{
		static TSharedRef<SDockTab> SpawnFacialAnimationBulkImporterTab(const FSpawnTabArgs& SpawnTabArgs)
		{
			const TSharedRef<SDockTab> Tab =
				SNew(SDockTab)
				.TabRole(ETabRole::NomadTab);

			Tab->SetContent(SNew(SFacialAnimationBulkImporter));

			return Tab;
		}
	};

	if (GetDefault<UEditorExperimentalSettings>()->bFacialAnimationImporter)
	{
		// Register a tab spawner so that our tab can be automatically restored from layout files
		FTabSpawnerEntry& TabSpawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FacialAnimationBulkImporterTabName, FOnSpawnTab::CreateStatic(&Local::SpawnFacialAnimationBulkImporterTab))
			.SetDisplayName(LOCTEXT("FacialAnimationBulkImporterTabTitle", "Facial Anim Importer"))
			.SetTooltipText(LOCTEXT("FacialAnimationBulkImporterTooltipText", "Open the Facial Animation Bulk Importer tab."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons..Import"));

		// register for when persona is loaded
		OnModulesChangedDelegate = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FFacialAnimationEditorModule::HandleModulesChanged);
	}

}

void FFacialAnimationEditorModule::ShutdownModule()
{
	FPersonaModule* PersonaModule = FModuleManager::GetModulePtr<FPersonaModule>(TEXT("Persona"));
	if (PersonaModule && OnPreviewSceneCreatedDelegate.IsValid())
	{
		PersonaModule->OnPreviewSceneCreated().Remove(OnPreviewSceneCreatedDelegate);
	}

	if (OnModulesChangedDelegate.IsValid())
	{
		FModuleManager::Get().OnModulesChanged().Remove(OnModulesChangedDelegate);
	}

	FGlobalTabmanager::Get()->UnregisterTabSpawner(FacialAnimationBulkImporterTabName);
}

#undef LOCTEXT_NAMESPACE
