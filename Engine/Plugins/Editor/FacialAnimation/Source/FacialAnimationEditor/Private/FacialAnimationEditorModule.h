// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"

class FFacialAnimationEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** Handle modules being loaded/unloaded */
	void HandleModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason);

private:
	/** Delegate for hooking into module loading */
	FDelegateHandle OnModulesChangedDelegate;

	/** Delegate for hooking into preview scene creation */
	FDelegateHandle OnPreviewSceneCreatedDelegate;
};
