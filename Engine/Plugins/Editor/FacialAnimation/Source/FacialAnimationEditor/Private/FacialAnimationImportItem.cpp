// Copyright Epic Games, Inc. All Rights Reserved.

#include "FacialAnimationImportItem.h"
#include "FacialAnimationBulkImporterSettings.h"
#include "Factories/SoundFactory.h"
#include "Modules/ModuleManager.h"
#include "Sound/SoundWave.h"
#include "Engine/CurveTable.h"
#include "FbxAnimUtils.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/Package.h"
#include "Utils.h"
#include "Logging/MessageLog.h"

#define LOCTEXT_NAMESPACE "SFacialAnimationBulkImporter"

bool FFacialAnimationImportItem::Import()
{
	if (!FbxFile.IsEmpty() && !WaveFile.IsEmpty())
	{
		return ImportCurvesEmbeddedInSoundWave();
	}

	return false;
}

USoundWave* FFacialAnimationImportItem::ImportSoundWave(const FString& InSoundWavePackageName, const FString& InSoundWaveAssetName, const FString& InWavFilename) const
{
	// Find or create the package to host the sound wave
	UPackage* const SoundWavePackage = CreatePackage(*InSoundWavePackageName);
	if (!SoundWavePackage)
	{
		FMessageLog("Import").Error(FText::Format(LOCTEXT("SoundWavePackageError", "Failed to create a sound wave package '{0}'."), FText::FromString(InSoundWavePackageName)));
		return nullptr;
	}

	// Make sure the destination package is loaded
	SoundWavePackage->FullyLoad();

	// We set the correct options in the constructor, so run the import silently
	USoundFactory* SoundWaveFactory = NewObject<USoundFactory>();
	SoundWaveFactory->SuppressImportDialogs();

	// Perform the actual import
	USoundWave* const SoundWave = ImportObject<USoundWave>(SoundWavePackage, *InSoundWaveAssetName, RF_Public | RF_Standalone, *InWavFilename, nullptr, SoundWaveFactory);
	if (!SoundWave)
	{
		FMessageLog("Import").Error(FText::Format(LOCTEXT("SoundWaveImportError", "Failed to import the sound wave asset '{0}.{1}' from '{2}'"), FText::FromString(InSoundWavePackageName), FText::FromString(InSoundWaveAssetName), FText::FromString(InWavFilename)));
		return nullptr;
	}

	// Compress to whatever formats the active target platforms want prior to saving the asset
	{
		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		if (TPM)
		{
			const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();
			for (ITargetPlatform* Platform : Platforms)
			{
				SoundWave->GetCompressedData(Platform->GetWaveFormat(SoundWave));
			}
		}
	}

	FAssetRegistryModule& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistry.AssetCreated(SoundWave);

	return SoundWave;
}

bool FFacialAnimationImportItem::ImportCurvesEmbeddedInSoundWave()
{
	// find/create our sound wave
	USoundWave* const SoundWave = ImportSoundWave(TargetPackageName, TargetAssetName, WaveFile);
	if (SoundWave)
	{
		// create our curve table internal to the sound wave
		static const FName InternalCurveTableName("InternalCurveTable");
		UCurveTable* NewCurveTable = NewObject<UCurveTable>(SoundWave, InternalCurveTableName);
		NewCurveTable->ClearFlags(RF_Public | RF_Standalone);
		NewCurveTable->SetFlags(NewCurveTable->GetFlags() | RF_Transactional);
		SoundWave->SetCurveData(NewCurveTable);
		SoundWave->SetInternalCurveData(NewCurveTable);

		// import curves from FBX
		float PreRollTime = 0.0f;
		if (FbxAnimUtils::ImportCurveTableFromNode(FbxFile, GetDefault<UFacialAnimationBulkImporterSettings>()->CurveNodeName, NewCurveTable, PreRollTime))
		{
			// we will need to add a curve to tell us the time we want to start playing audio
			FRichCurve& AudioCurve = NewCurveTable->AddRichCurve(TEXT("Audio"));
			AudioCurve.AddKey(PreRollTime, 1.0f);

			return true;
		}
	}

	return false;
}


#undef LOCTEXT_NAMESPACE
