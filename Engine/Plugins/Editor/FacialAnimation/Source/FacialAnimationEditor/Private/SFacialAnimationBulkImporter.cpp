// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFacialAnimationBulkImporter.h"
#include "DetailsViewArgs.h"
#include "FacialAnimationImportItem.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Widgets/SBoxPanel.h"
#include "Misc/Paths.h"
#include "FacialAnimationBulkImporterSettings.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PackageName.h"

#define LOCTEXT_NAMESPACE "SFacialAnimationBulkImporter"

void SFacialAnimationBulkImporter::Construct(const FArguments& InArgs)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(GetMutableDefault<UFacialAnimationBulkImporterSettings>());

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			DetailsView
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(4.0f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			.ContentPadding(FMargin(6, 2))
			.IsEnabled(this, &SFacialAnimationBulkImporter::IsImportButtonEnabled)
			.OnClicked(this, &SFacialAnimationBulkImporter::HandleImportClicked)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Text(LOCTEXT("ImportAllButton", "Import All"))
			]
		]
	];
}

bool SFacialAnimationBulkImporter::IsImportButtonEnabled() const
{
	const UFacialAnimationBulkImporterSettings* FacialAnimationBulkImporterSettings = GetDefault<UFacialAnimationBulkImporterSettings>();
	return !FacialAnimationBulkImporterSettings->SourceImportPath.Path.IsEmpty() && !FacialAnimationBulkImporterSettings->TargetImportPath.Path.IsEmpty();
}

FReply SFacialAnimationBulkImporter::HandleImportClicked()
{
	// Iterate over the chosen source directory adding items to import
	struct FFbxVisitor : public IPlatformFile::FDirectoryVisitor
	{
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			if (!bIsDirectory)
			{
				FString FbxFilename(FilenameOrDirectory);
				if (FPaths::GetExtension(FbxFilename).Equals(TEXT("FBX"), ESearchCase::IgnoreCase))
				{
					// check for counterpart wave file
					FString WaveFilename = FPaths::ChangeExtension(FbxFilename, TEXT("WAV"));

					IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
					if (!PlatformFile.FileExists(*WaveFilename))
					{
						// no wave file so clear it out, this will be a standalone animation
						WaveFilename.Empty();
					}

					// build target package/asset name
					FString TargetAssetName = FPaths::GetBaseFilename(FbxFilename);

					const UFacialAnimationBulkImporterSettings* FacialAnimationBulkImporterSettings = GetDefault<UFacialAnimationBulkImporterSettings>();
					FString CurrentDirectory = FPaths::GetPath(FbxFilename);
					FString PartialPath = CurrentDirectory.RightChop(FacialAnimationBulkImporterSettings->SourceImportPath.Path.Len());
					FString TargetPackageName = FacialAnimationBulkImporterSettings->TargetImportPath.Path / PartialPath / TargetAssetName;

					FString Filename;
					if (FPackageName::TryConvertLongPackageNameToFilename(TargetPackageName, Filename))
					{
						ItemsToImport.Add({ FbxFilename, WaveFilename, TargetPackageName, TargetAssetName });
					}
				}
			}
			return true;
		}

		TArray<FFacialAnimationImportItem> ItemsToImport;
	};

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FFbxVisitor Visitor;
	PlatformFile.IterateDirectoryRecursively(*GetDefault<UFacialAnimationBulkImporterSettings>()->SourceImportPath.Path, Visitor);
	
	// @TODO: check for a valid skeleton if we have standalone animations to import

	for (FFacialAnimationImportItem& ImportItem : Visitor.ItemsToImport)
	{
		ImportItem.Import();
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
