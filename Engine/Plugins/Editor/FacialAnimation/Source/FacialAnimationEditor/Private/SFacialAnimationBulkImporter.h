// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SFacialAnimationBulkImporter : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFacialAnimationBulkImporter)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	bool IsImportButtonEnabled() const;

	FReply HandleImportClicked();
};
