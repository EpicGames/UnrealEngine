// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FFacialAnimationImportItem
{
	bool Import();

private:
	class USoundWave* ImportSoundWave(const FString& InSoundWavePackageName, const FString& InSoundWaveAssetName, const FString& InWavFilename) const;
	bool ImportCurvesEmbeddedInSoundWave();

public:
	FString FbxFile;
	FString WaveFile;
	FString TargetPackageName;
	FString TargetAssetName;
};